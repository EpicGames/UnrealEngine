// Copyright Epic Games, Inc. All Rights Reserved.

#include "ITextureShareSDK.h"
#include "TextureShareSDKObject.h"

#include "ITextureShareCore.h"
#include "ITextureShareCoreAPI.h"

/////////////////////////////////////////////////////////////////////////////////////////////
// SDK DLL (code copied from UE)
//////////////////////////////////////////////////////////////////////////////////////////////
#include "Modules/ModuleManager.h"
#include "Windows/AllowWindowsPlatformTypes.h"

BOOL WINAPI DllMain(HINSTANCE hinstDLL, DWORD fdwReason, LPVOID lpReserved)
{
	// Perform actions based on the reason for calling.
	switch (fdwReason)
	{
	case DLL_PROCESS_ATTACH:
		// Initialize once for each new process.
		// Return FALSE to fail DLL load.
		break;

	case DLL_THREAD_ATTACH:
		// Do thread-specific initialization.
		break;

	case DLL_THREAD_DETACH:
		// Do thread-specific cleanup.
		break;

	case DLL_PROCESS_DETACH:
		// Perform any necessary cleanup.
		break;
	}
	return TRUE;  // Successful DLL_PROCESS_ATTACH.
}

// If we don't have to implement a module, or application, and we can save 200kb by not depending on "Projects" module, so borrow some code from UnrealGame.cpp to bypass it 
TCHAR GInternalProjectName[64] = TEXT("");
IMPLEMENT_FOREIGN_ENGINE_DIR()

#include "Windows/HideWindowsPlatformTypes.h"

//////////////////////////////////////////////////////////////////////////////////////////////
namespace TextureShareInterfaceHelpers
{
	static inline ITextureShareCoreAPI& TextureShareCoreAPI()
	{
		static ITextureShareCoreAPI& TextureShareCoreAPISingleton = ITextureShareCore::Get().GetTextureShareCoreAPI();
		return TextureShareCoreAPISingleton;
	}

	static TMap<FString, TSharedPtr<FTextureShareSDKObject, ESPMode::ThreadSafe>> SDKObjects;
};
using namespace TextureShareInterfaceHelpers;

/////////////////////////////////////////////////////////////////////////////////////////////
// ITextureShareSDK
//////////////////////////////////////////////////////////////////////////////////////////////
void ITextureShareSDK::GetProcessDesc(TDataOutput<FTextureShareCoreObjectProcessDesc>& OutProcessDesc)
{
	OutProcessDesc = TextureShareCoreAPI().GetProcessDesc();
}

void ITextureShareSDK::SetProcessName(const wchar_t* InProcessId)
{
	TextureShareCoreAPI().SetProcessName(InProcessId);
}

bool ITextureShareSDK::SetProcessDeviceType(const ETextureShareDeviceType InDeviceType)
{
	return TextureShareCoreAPI().SetProcessDeviceType(InDeviceType);
}

void ITextureShareSDK::RemoveUnusedResources(const uint32 InMilisecondsTimeOut)
{
	TextureShareCoreAPI().RemoveUnusedResources(InMilisecondsTimeOut);
}

//////////////////////////////////////////////////////////////////////////////////////////////
ITextureShareSDKObject* ITextureShareSDK::GetOrCreateObject(const wchar_t* InShareName)
{
	const FString ShareNameLwr = FString(InShareName).ToLower();

	TSharedPtr<FTextureShareSDKObject, ESPMode::ThreadSafe> const* ExistRef = SDKObjects.Find(ShareNameLwr);
	if (ExistRef)
	{
		return ExistRef->Get();
	}

	TSharedPtr<ITextureShareCoreObject, ESPMode::ThreadSafe> ShareObject = TextureShareCoreAPI().GetOrCreateCoreObject(InShareName);
	if (ShareObject.IsValid())
	{
		TSharedPtr<FTextureShareSDKObject> NewObjectInterface = MakeShared<FTextureShareSDKObject>(ShareObject);
		if (NewObjectInterface.IsValid())
		{
			SDKObjects.Emplace(ShareNameLwr, NewObjectInterface);

			return NewObjectInterface.Get();
		}
	}

	return nullptr;
}

bool ITextureShareSDK::IsObjectExist(const wchar_t* InShareName)
{
	return SDKObjects.Contains(FString(InShareName).ToLower());
}

bool ITextureShareSDK::RemoveObject(const wchar_t* InShareName)
{
	return SDKObjects.Remove(FString(InShareName).ToLower()) > 0;
}

bool ITextureShareSDK::GetInterprocessObjects(TDataOutput<TArraySerializable<FTextureShareCoreObjectDesc>>& OutInterprocessObjects, const wchar_t* InShareName)
{
	const FString ShareName = InShareName ? InShareName : TEXT("");

	TArraySerializable<FTextureShareCoreObjectDesc> InterprocessObjects;
	if (TextureShareCoreAPI().GetInterprocessObjects(ShareName, InterprocessObjects))
	{
		OutInterprocessObjects = InterprocessObjects;

		return true;
	}

	return false;
}
