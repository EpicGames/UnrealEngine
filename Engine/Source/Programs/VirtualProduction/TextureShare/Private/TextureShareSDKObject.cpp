// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureShareSDKObject.h"

#include "ITextureShareCore.h"
#include "ITextureShareCoreAPI.h"

#include "ITextureShareCoreObject.h"

//////////////////////////////////////////////////////////////////////////////////////////////
namespace TextureShareSDKObjectHelpers
{
	static inline ITextureShareCoreAPI& GetTextureShareCoreAPI()
	{
		static ITextureShareCoreAPI& TextureShareCoreAPISingleton = ITextureShareCore::Get().GetTextureShareCoreAPI();
		return TextureShareCoreAPISingleton;
	}
};
using namespace TextureShareSDKObjectHelpers;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareSDKObject
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareSDKObject::FTextureShareSDKObject(const TSharedPtr<ITextureShareCoreObject, ESPMode::ThreadSafe>& InCoreObject)
	: CoreObject(InCoreObject)
{
	check(CoreObject.IsValid());
}

FTextureShareSDKObject::~FTextureShareSDKObject()
{
	if (CoreObject.IsValid())
	{
		CoreObject->RemoveObject();
	}
}
//////////////////////////////////////////////////////////
// FTextureShareSDKObject: State
//////////////////////////////////////////////////////////
const wchar_t* FTextureShareSDKObject::GetName() const
{
	check(CoreObject.IsValid());
	return FTCHARToWChar(*CoreObject->GetObjectDesc().ShareName).Get();
}

void FTextureShareSDKObject::GetObjectDesc(TDataOutput<FTextureShareCoreObjectDesc>& OutObjectDesc) const
{
	check(CoreObject.IsValid());

	OutObjectDesc = CoreObject->GetObjectDesc();
}

void FTextureShareSDKObject::GetObjectDesc_RenderThread(TDataOutput<FTextureShareCoreObjectDesc>& OutObjectDesc) const
{
	check(CoreObject.IsValid());

	OutObjectDesc = CoreObject->GetObjectDesc_RenderThread();
}

bool FTextureShareSDKObject::IsActive() const
{
	check(CoreObject.IsValid());

	return CoreObject->IsActive();
}

bool FTextureShareSDKObject::IsActive_RenderThread() const
{
	check(CoreObject.IsValid());

	return CoreObject->IsActive_RenderThread();
}

bool FTextureShareSDKObject::IsFrameSyncActive() const
{
	check(CoreObject.IsValid());

	return CoreObject->IsFrameSyncActive();
}

bool FTextureShareSDKObject::IsFrameSyncActive_RenderThread() const
{
	check(CoreObject.IsValid());

	return CoreObject->IsFrameSyncActive_RenderThread();
}


bool FTextureShareSDKObject::FindSkippedSyncStep(const ETextureShareSyncStep InSyncStep, ETextureShareSyncStep& OutSkippedSyncStep) const
{
	check(CoreObject.IsValid());

	return CoreObject->FindSkippedSyncStep(InSyncStep, OutSkippedSyncStep);
}

bool FTextureShareSDKObject::FindSkippedSyncStep_RenderThread(const ETextureShareSyncStep InSyncStep, ETextureShareSyncStep& OutSkippedSyncStep) const
{
	check(CoreObject.IsValid());

	return CoreObject->FindSkippedSyncStep_RenderThread(InSyncStep, OutSkippedSyncStep);
}

//////////////////////////////////////////////////////////
// FTextureShareSDKObject: Settings
//////////////////////////////////////////////////////////
bool FTextureShareSDKObject::SetProcessId(const wchar_t* InProcessId)
{
	return CoreObject.IsValid() && CoreObject->SetProcessId(InProcessId);
}

bool FTextureShareSDKObject::SetDeviceType(const ETextureShareDeviceType InDeviceType)
{
	return CoreObject.IsValid() && CoreObject->SetDeviceType(InDeviceType);
}

bool FTextureShareSDKObject::SetSyncSetting(const TDataInput<FTextureShareCoreSyncSettings>& InSyncSetting)
{
	check(CoreObject.IsValid());
	return CoreObject->SetSyncSettings(*TDataVariable<FTextureShareCoreSyncSettings>(InSyncSetting));
}

void FTextureShareSDKObject::GetSyncSetting(TDataOutput<FTextureShareCoreSyncSettings>& OutSyncSettings) const
{
	check(CoreObject.IsValid());
	OutSyncSettings = CoreObject->GetSyncSettings();
}

void FTextureShareSDKObject::GetFrameSyncSettings(const ETextureShareFrameSyncTemplate InType, TDataOutput<FTextureShareCoreFrameSyncSettings>& OutFrameSyncSettings) const
{
	check(CoreObject.IsValid());
	OutFrameSyncSettings = CoreObject->GetFrameSyncSettings(InType);
}

//////////////////////////////////////////////////////////
// FTextureShareSDKObject: Session
//////////////////////////////////////////////////////////
bool FTextureShareSDKObject::BeginSession()
{
	return CoreObject.IsValid() && CoreObject->BeginSession();
}

bool FTextureShareSDKObject::EndSession()
{
	return CoreObject.IsValid() && CoreObject->EndSession();
}

bool FTextureShareSDKObject::IsSessionActive() const
{
	return CoreObject.IsValid() && CoreObject->IsSessionActive();
}

//////////////////////////////////////////////////////////
// FTextureShareSDKObject: Thread sync support
//////////////////////////////////////////////////////////
bool FTextureShareSDKObject::LockThreadMutex(const ETextureShareThreadMutex InThreadMutex, bool bForceLockNoWait)
{
	return CoreObject.IsValid() && CoreObject->LockThreadMutex(InThreadMutex, bForceLockNoWait);
}

bool FTextureShareSDKObject::UnlockThreadMutex(const ETextureShareThreadMutex InThreadMutex)
{
	return CoreObject.IsValid() && CoreObject->UnlockThreadMutex(InThreadMutex);
}

bool FTextureShareSDKObject::IsBeginFrameSyncActive() const
{
	return CoreObject.IsValid() && CoreObject->IsBeginFrameSyncActive();
}

bool FTextureShareSDKObject::IsBeginFrameSyncActive_RenderThread() const
{
	return CoreObject.IsValid() && CoreObject->IsBeginFrameSyncActive_RenderThread();
}

//////////////////////////////////////////////////////////
// FTextureShareSDKObject: Interprocess Synchronization
//////////////////////////////////////////////////////////
bool FTextureShareSDKObject::BeginFrameSync()
{
	return CoreObject.IsValid() && CoreObject->BeginFrameSync();
}

bool FTextureShareSDKObject::FrameSync(const ETextureShareSyncStep InSyncStep)
{
	return CoreObject.IsValid() && CoreObject->FrameSync(InSyncStep);
}

bool FTextureShareSDKObject::EndFrameSync()
{
	return CoreObject.IsValid() && CoreObject->EndFrameSync();
}

///////////////////////////////////////////////////////////////////////////
bool FTextureShareSDKObject::BeginFrameSync_RenderThread()
{
	return CoreObject.IsValid() && CoreObject->BeginFrameSync_RenderThread();
}

bool FTextureShareSDKObject::FrameSync_RenderThread(const ETextureShareSyncStep InSyncStep)
{
	return CoreObject.IsValid() && CoreObject->FrameSync_RenderThread(InSyncStep);
}

bool FTextureShareSDKObject::EndFrameSync_RenderThread()
{
	return CoreObject.IsValid() && CoreObject->EndFrameSync_RenderThread();
}

void FTextureShareSDKObject::GetConnectedInterprocessObjects(TDataOutput<TArraySerializable<FTextureShareCoreObjectDesc>>& OutConnectedObjects) const
{
	check(CoreObject.IsValid());
	OutConnectedObjects = CoreObject->GetConnectedInterprocessObjects();
}

//////////////////////////////////////////////////////////
// FTextureShareSDKObject: Data Containers
//////////////////////////////////////////////////////////
void FTextureShareSDKObject::GetData(TDataOutput<FTextureShareCoreData>& OutObjectData)
{
	check(CoreObject.IsValid());
	OutObjectData = CoreObject->GetData();
}

void FTextureShareSDKObject::SetData(const TDataInput<FTextureShareCoreData>& InObjectData)
{
	check(CoreObject.IsValid());
	CoreObject->GetData() = *TDataVariable<FTextureShareCoreData>(InObjectData);
}

void FTextureShareSDKObject::GetProxyData_RenderThread(TDataOutput<FTextureShareCoreProxyData>& OutObjectProxyData)
{
	check(CoreObject.IsValid());
	OutObjectProxyData = CoreObject->GetProxyData_RenderThread();
}

void FTextureShareSDKObject::SetProxyData_RenderThread(const TDataInput<FTextureShareCoreProxyData>& InObjectProxyData)
{
	check(CoreObject.IsValid());
	CoreObject->GetProxyData_RenderThread() = *TDataVariable<FTextureShareCoreProxyData>(InObjectProxyData);
}

void FTextureShareSDKObject::GetReceivedData(TDataOutput<TArraySerializable<FTextureShareCoreObjectData>>& OutReceivedObjectsData) const
{
	check(CoreObject.IsValid());
	OutReceivedObjectsData = CoreObject->GetReceivedData();
}

void FTextureShareSDKObject::GetReceivedProxyData_RenderThread(TDataOutput<TArraySerializable<FTextureShareCoreObjectProxyData>>& OutObjectsProxyData) const
{
	check(CoreObject.IsValid());
	OutObjectsProxyData = CoreObject->GetReceivedProxyData_RenderThread();
}

//////////////////////////////////////////////////////////
// FTextureShareSDKObject: Access to shared resources
//////////////////////////////////////////////////////////
ID3D11Texture2D* FTextureShareSDKObject::OpenSharedResourceD3D11(ID3D11Device* pD3D11Device, const TDataInput<FTextureShareCoreResourceDesc>& InResourceDesc)
{
	TSharedPtr<ITextureShareCoreD3D11ResourcesCache, ESPMode::ThreadSafe> D3D11Resources = GetTextureShareCoreAPI().GetD3D11ResourcesCache();
	if (CoreObject.IsValid() && D3D11Resources.IsValid())
	{
		FTextureShareCoreResourceDesc ResourceDesc(*TDataVariable<FTextureShareCoreResourceDesc>(InResourceDesc));
		FTextureShareCoreResourceHandle ResourceHandle;
		if (ImplFindResource(ResourceDesc, ResourceHandle))
		{
			return D3D11Resources->OpenSharedResource(CoreObject->GetObjectDesc(), pD3D11Device, ResourceHandle);
		}
	}

	return nullptr;
}

ID3D12Resource* FTextureShareSDKObject::OpenSharedResourceD3D12(ID3D12Device* pD3D12Device, const TDataInput<FTextureShareCoreResourceDesc>& InResourceDesc)
{
	TSharedPtr<ITextureShareCoreD3D12ResourcesCache, ESPMode::ThreadSafe> D3D12Resources = GetTextureShareCoreAPI().GetD3D12ResourcesCache();
	if (CoreObject.IsValid() && D3D12Resources.IsValid())
	{
		FTextureShareCoreResourceDesc ResourceDesc(*TDataVariable<FTextureShareCoreResourceDesc>(InResourceDesc));
		FTextureShareCoreResourceHandle ResourceHandle;
		if (ImplFindResource(ResourceDesc, ResourceHandle))
		{
			return D3D12Resources->OpenSharedResource(CoreObject->GetObjectDesc(), pD3D12Device, ResourceHandle);
		}
	}

	return nullptr;
}

bool FTextureShareSDKObject::OpenSharedResourceVulkan(const TDataInput<FTextureShareDeviceVulkanContext>& InDeviceVulkanContext, const TDataInput<FTextureShareCoreResourceDesc>& InResourceDesc, TDataOutput<FTextureShareDeviceVulkanResource>& OutVulkanResource)
{
	TSharedPtr<ITextureShareCoreVulkanResourcesCache, ESPMode::ThreadSafe> VulkanResources = GetTextureShareCoreAPI().GetVulkanResourcesCache();
	if (CoreObject.IsValid() && VulkanResources.IsValid())
	{
		FTextureShareCoreResourceDesc ResourceDesc(*TDataVariable<FTextureShareCoreResourceDesc>(InResourceDesc));
		FTextureShareCoreResourceHandle ResourceHandle;
		if (ImplFindResource(ResourceDesc, ResourceHandle))
		{
			FTextureShareDeviceVulkanContext DeviceVulkanContext(*TDataVariable<FTextureShareDeviceVulkanContext>(InDeviceVulkanContext));
			FTextureShareDeviceVulkanResource VulkanResource = VulkanResources->OpenSharedResource(CoreObject->GetObjectDesc(), DeviceVulkanContext, ResourceHandle);
			if (VulkanResource.IsValid())
			{
				OutVulkanResource = VulkanResource;

				return true;
			}
		}
	}

	return false;
}

//////////////////////////////////////////////////////////
bool FTextureShareSDKObject::ImplFindResource(const FTextureShareCoreResourceDesc& InResourceDesc, FTextureShareCoreResourceHandle& OutResourceHandle) const
{
	if (CoreObject.IsValid())
	{
		const TArraySerializable<FTextureShareCoreObjectProxyData> ReceivedObjectsProxyData = CoreObject->GetReceivedProxyData_RenderThread();
		for (const FTextureShareCoreObjectProxyData& ObjectProxyDataIt : ReceivedObjectsProxyData)
		{
			if (const FTextureShareCoreResourceHandle* ExistResource = ObjectProxyDataIt.ProxyData.ResourceHandles.FindByEqualsFunc(InResourceDesc))
			{
				// Get resource from any process
				OutResourceHandle = *ExistResource;
				return true;
			}
		}
	}

	return false;
}
