// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShareSDKObject.h"

class ITextureShareCoreObject;

/**
 * TextureShareSDK object impl
 */
class TEXTURESHARESDK_API FTextureShareSDKObject
	: public ITextureShareSDKObject
{
public:
	FTextureShareSDKObject(const TSharedPtr<ITextureShareCoreObject, ESPMode::ThreadSafe>& InCoreObject);
	virtual ~FTextureShareSDKObject();

public:
	///////////////////////// State /////////////////////////
	virtual const wchar_t* GetName() const override;

	virtual void GetObjectDesc(TDataOutput<FTextureShareCoreObjectDesc>& OutObjectDesc) const override;
	virtual void GetObjectDesc_RenderThread(TDataOutput<FTextureShareCoreObjectDesc>& OutObjectDesc) const override;

	virtual bool IsActive() const override;
	virtual bool IsActive_RenderThread() const override;

	virtual bool IsFrameSyncActive() const override;
	virtual bool IsFrameSyncActive_RenderThread() const override;

	virtual bool IsBeginFrameSyncActive() const override;
	virtual bool IsBeginFrameSyncActive_RenderThread() const override;

public:
	///////////////////////// Settings /////////////////////////
	virtual bool SetProcessId(const wchar_t* InProcessId) override;
	virtual bool SetDeviceType(const ETextureShareDeviceType InDeviceType) override;
	virtual bool SetSyncSetting(const TDataInput<FTextureShareCoreSyncSettings>& InSyncSetting) override;
	virtual void GetSyncSetting(TDataOutput<FTextureShareCoreSyncSettings>& OutSyncSettings) const override;
	virtual void GetFrameSyncSettings(const ETextureShareFrameSyncTemplate InType, TDataOutput<FTextureShareCoreFrameSyncSettings>& OutFrameSyncSettings) const override;

	virtual bool FindSkippedSyncStep(const ETextureShareSyncStep InSyncStep, ETextureShareSyncStep& OutSkippedSyncStep) const override;
	virtual bool FindSkippedSyncStep_RenderThread(const ETextureShareSyncStep InSyncStep, ETextureShareSyncStep& OutSkippedSyncStep) const override;

public:
	///////////////////////// Session /////////////////////////
	virtual bool BeginSession() override;
	virtual bool EndSession() override;
	virtual bool IsSessionActive() const override;

public:
	///////////////////////// Thread sync support /////////////////////////
	virtual bool LockThreadMutex(const ETextureShareThreadMutex InThreadMutex, bool bForceLockNoWait = false) override;
	virtual bool UnlockThreadMutex(const ETextureShareThreadMutex InThreadMutex) override;

public:
	///////////////////////// Interprocess Synchronization /////////////////////////
	virtual bool BeginFrameSync() override;
	virtual bool FrameSync(const ETextureShareSyncStep InSyncStep) override;
	virtual bool EndFrameSync() override;
	virtual bool BeginFrameSync_RenderThread() override;
	virtual bool FrameSync_RenderThread(const ETextureShareSyncStep InSyncStep) override;
	virtual bool EndFrameSync_RenderThread() override;
	virtual void GetConnectedInterprocessObjects(TDataOutput<TArraySerializable<FTextureShareCoreObjectDesc>>& OutObjectList) const override;

public:
	///////////////////////// Data Containers /////////////////////////
	virtual void GetData(TDataOutput<FTextureShareCoreData>& OutObjectData) override;
	virtual void SetData(const TDataInput<FTextureShareCoreData>& InObjectData) override;
	virtual void GetProxyData_RenderThread(TDataOutput<FTextureShareCoreProxyData>& OutObjectProxyData) override;
	virtual void SetProxyData_RenderThread(const TDataInput<FTextureShareCoreProxyData>& InObjectProxyData) override;
	virtual void GetReceivedData(TDataOutput<TArraySerializable<FTextureShareCoreObjectData>>& OutReceivedObjectsData) const override;
	virtual void GetReceivedProxyData_RenderThread(TDataOutput<TArraySerializable<FTextureShareCoreObjectProxyData>>& OutObjectsProxyData) const override;

public:
	///////////////////////// Access to shared resources /////////////////////////
	virtual ID3D11Texture2D* OpenSharedResourceD3D11(ID3D11Device* pD3D11Device, const TDataInput<FTextureShareCoreResourceDesc>& InResourceDesc) override;
	virtual ID3D12Resource*  OpenSharedResourceD3D12(ID3D12Device* pD3D12Device, const TDataInput<FTextureShareCoreResourceDesc>& InResourceDesc) override;

	virtual  bool OpenSharedResourceVulkan(const TDataInput<FTextureShareDeviceVulkanContext>& InDeviceVulkanContext, const TDataInput<FTextureShareCoreResourceDesc>& InResourceDesc, TDataOutput<FTextureShareDeviceVulkanResource>& OutVulkanResource) override;

private:
	bool ImplFindResource(const struct FTextureShareCoreResourceDesc& InResourceDesc, struct FTextureShareCoreResourceHandle& OutResourceHandle) const;

private:
	TSharedPtr<ITextureShareCoreObject, ESPMode::ThreadSafe> const CoreObject;
};

