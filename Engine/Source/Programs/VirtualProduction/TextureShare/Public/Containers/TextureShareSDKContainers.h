// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/UnrealEngine/TextureShareSDKUnrealEngineArray.h"
#include "Containers/UnrealEngine/TextureShareSDKUnrealEngineContainers.h"
#include "Containers/UnrealEngine/TextureShareSDKUnrealEngineEnums_PixelFormat.h"
#include "Containers/UnrealEngine/TextureShareSDKUnrealEngineMath.h"
#include "Containers/UnrealEngine/TextureShareSDKUnrealEngineString.h"

#include "Containers/TextureShareCoreEnums.h"
#include "Containers/TextureShareCoreContainers.h"

#include "Containers/TextureShareCoreContainers_DeviceD3D11.h"
#include "Containers/TextureShareCoreContainers_DeviceD3D12.h"
#include "Containers/TextureShareCoreContainers_DeviceVulkan.h"

#ifndef TEXTURESHARESDK_API

// Support SDK user side DLL functions import
#define TEXTURESHARESDK_API __declspec(dllimport)

#endif
