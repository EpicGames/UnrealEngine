// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/UnrealEngine/TextureShareSDKUnrealEngineTypes.h"

#ifndef __UNREAL__

#include <vector>

enum { INDEX_NONE = -1 };

//@todo: implement better logic for std::vector vs TArray (this is too simplistic now)

/**
 * Implementation of TArray on the SDK ext app side (reflect this data type)
 */
template<typename InElementType>
struct TArray
	: public std::vector<InElementType>
{
	typedef InElementType ElementType;
	typedef         int32 SizeType;

	/**
	 * Finds element within the array.
	 *
	 * @param Item Item to look for.
	 * @returns Index of the found element. INDEX_NONE otherwise.
	 * @see FindLast, FindLastByPredicate
	 */
	inline SizeType Find(const ElementType& Item) const
	{
		const SizeType Count = Num();

		for(int Index = 0; Index < Count; Index++)
		{
			if (this->operator[](Index) == Item)
			{
				return Index;
			}
		}
		return INDEX_NONE;
	}

	/**
	 * Helper function for returning a typed pointer to the first array entry.
	 *
	 * @returns Pointer to first array entry or nullptr if ArrayMax == 0.
	 */
	inline ElementType* GetData()
	{
		return (ElementType*)this->data();
	}

	/**
	 * Helper function for returning a typed pointer to the first array entry.
	 *
	 * @returns Pointer to first array entry or nullptr if ArrayMax == 0.
	 */
	inline const ElementType* GetData() const
	{
		return (const ElementType*)this->data();
	}

	/**
	 * Empties the array. It calls the destructors on held items if needed.
	 */
	inline void Empty()
	{
		this->clear();
	}

	/**
	 * Same as empty, but doesn't change memory allocations, unless the new size is larger than
	 * the current array. It calls the destructors on held items if needed and then zeros the ArrayNum.
	 */
	void Reset()
	{
		this->clear();
	}

	/**
	 * Returns true if the array is empty and contains no elements.
	 *
	 * @returns True if the array is empty.
	 * @see Num
	 */
	inline bool IsEmpty() const
	{
		return Num() == 0;
	}

	/**
	 * Returns number of elements in array.
	 *
	 * @returns Number of elements in array.
	 */
	inline SizeType Num() const
	{
		const SizeType Index = (SizeType)(this->size());

		return Index;
	}

	/**
	 * Adds a new item to the end of the array, possibly reallocating the whole array to fit.
	 *
	 * @param Item The item to add
	 * @return Index to the new item
	 * @see AddDefaulted, AddUnique, AddZeroed, Append, Insert
	 */
	inline SizeType Add(const ElementType& Item)
	{
		const SizeType Index = Num();

		this->push_back(Item);

		return Index;
	}

	inline SizeType AddUnique(const ElementType& Item)
	{
		SizeType ItemIndex = Find(Item);
		if (ItemIndex == INDEX_NONE)
		{
			// Add unique
			ItemIndex = Num();
			this->push_back(Item);
		}

		return ItemIndex;
	}

	inline void Insert(const ElementType& Item, const int32 Index)
	{
		this->insert(this->begin() + Index, Item);
	}

	/**
	 * Adds new items to the end of the array, possibly reallocating the whole
	 * array to fit. The new items will be zeroed.
	 *
	 * Caution, AddZeroed() will create elements without calling the
	 * constructor and this is not appropriate for element types that require
	 * a constructor to function properly.
	 *
	 * @param  Count  The number of new items to add.
	 * @return Index to the first of the new items.
	 * @see Add, AddDefaulted, AddUnique, Append, Insert
	 */
	inline SizeType AddZeroed(SizeType Count = 1)
	{
		if (Count > 0)
		{
			const SizeType Index = Num();

			this->resize(Index + Count);

			// set to zero all new elements
			std::memset((uint8*)(GetData()) + sizeof(ElementType) * Index, 0, sizeof(ElementType) * Count);

			return Index;
		}

		return INDEX_NONE;
	}

	/**
	 * Adds new items to the end of the array, possibly reallocating the whole
	 * array to fit. The new items will be default-constructed.
	 *
	 * @param  Count  The number of new items to add.
	 * @return Index to the first of the new items.
	 * @see Add, AddZeroed, AddUnique, Append, Insert
	 */
	inline SizeType AddDefaulted(SizeType Count = 1)
	{
		if (Count > 0)
		{
			const SizeType Index = Num();

			this->resize(Index + Count);

			return Index;
		}

		return INDEX_NONE;
	}

	/**
	 * Removes an element (or elements) at given location optionally shrinking
	 * the array.
	 *
	 * @param Index Location in array of the element to remove.
	 */
	inline void RemoveAt(SizeType Index, SizeType Count = 1)
	{
		if (Index >= 0 && Index < Num())
		{
			if (Count > 1)
			{
				this->erase(this->begin() + Index, this->begin() + Index + Count);
			}
			else
			{
				this->erase(this->begin() + Index);
			}
		}
	}
};
#endif
