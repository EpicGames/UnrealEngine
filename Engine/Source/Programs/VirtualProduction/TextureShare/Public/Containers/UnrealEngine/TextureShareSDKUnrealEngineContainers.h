// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/UnrealEngine/TextureShareSDKUnrealEngineTypes.h"

#ifndef __UNREAL__
/**
 * Implementation of UnrealEngine containers on the SDK side
 */


/**
 * SDK ext app type (reflect UE struct FGuid)
 */
struct FGuid
{
	uint32 A = 0;
	uint32 B = 0;
	uint32 C = 0;
	uint32 D = 0;

public:
	inline bool operator==(const FGuid& In) const
	{
		return A == In.A && B == In.B && C == In.C && D == In.D;
	}
};

/**
 * SDK ext app type (reflect UE struct FIntPoint)
 */
struct FIntPoint
{
	int32 X = 0;
	int32 Y = 0;

public:
	FIntPoint() = default;
	FIntPoint(int32 InX, int32 InY)
		: X(InX), Y(InY)
	{ }

public:
	static FIntPoint ZeroValue;
};

/**
 * SDK ext app type (reflect UE struct FIntRect)
 */
struct FIntRect
{
	FIntPoint Min;
	FIntPoint Max;

public:
	FIntRect() = default;
	FIntRect(const FIntPoint& InMin, const FIntPoint& InMax)
		: Min(InMin), Max(InMax)
	{ }
};

/**
 * SDK ext app type (reflect UE struct FVector)
 */
struct FVector
{
	double X = 0;
	double Y = 0;
	double Z = 0;

public:
	FVector() = default;
	FVector(double InX, double InY, double InZ)
		: X(InX), Y(InY), Z(InZ)
	{ }

public:
	static FVector ZeroVector;
};

/**
 * SDK ext app type (reflect UE struct FQuat)
 */
struct FQuat
{
	double X = 0;
	double Y = 0;
	double Z = 0;
	double W = 1;

public:
	FQuat() = default;
	FQuat(double InX, double InY, double InZ, double InW)
		: X(InX), Y(InY), Z(InZ), W(InW)
	{ }
};

/**
 * SDK ext app type (reflect UE struct FVector2D)
 */
struct FVector2D
{
	double X = 0;
	double Y = 0;

public:
	FVector2D() = default;
	FVector2D(double InX, double InY)
		: X(InX), Y(InY)
	{ }
};

/**
 * SDK ext app type (reflect UE struct FMatrix)
 */
struct FMatrix
{
	double M[4][4] = { {1,0,0,0}, {0,1,0,0}, {0,0,1,0}, {0,0,0,1} };

public:
	FMatrix() = default;

public:
	static FMatrix Identity;
};

/**
 * SDK ext app type (reflect UE struct FRotator)
 */
struct FRotator
{
	/** Rotation around the right axis (around Y axis), Looking up and down (0=Straight Ahead, +Up, -Down) */
	double Pitch = 0;

	/** Rotation around the up axis (around Z axis), Running in circles 0=East, +North, -South. */
	double Yaw = 0;

	/** Rotation around the forward axis (around X axis), Tilting your head, 0=Straight, +Clockwise, -CCW. */
	double Roll = 0;

public:
	FRotator() = default;
	FRotator(double InPitch, double InYaw, double InRoll)
		: Pitch(InPitch), Yaw(InYaw), Roll(InRoll)
	{ }

public:
	static FRotator ZeroRotator;
};
#endif
