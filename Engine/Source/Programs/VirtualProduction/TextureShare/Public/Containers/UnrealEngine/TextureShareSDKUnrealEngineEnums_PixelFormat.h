// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef __UNREAL__

/**
 * Reflect UE EPixelFormat enum on the SDK ext app side
 * The code below is copied from the UE
 */
enum EPixelFormat : uint8
{
	PF_Unknown = 0,
	PF_A32B32G32R32F = 1,
	PF_B8G8R8A8 = 2,
	PF_G8 = 3,
	PF_G16 = 4,
	PF_DXT1 = 5,
	PF_DXT3 = 6,
	PF_DXT5 = 7,
	PF_UYVY = 8,
	PF_FloatRGB = 9,
	PF_FloatRGBA = 10,
	PF_DepthStencil = 11,
	PF_ShadowDepth = 12,
	PF_R32_FLOAT = 13,
	PF_G16R16 = 14,
	PF_G16R16F = 15,
	PF_G16R16F_FILTER = 16,
	PF_G32R32F = 17,
	PF_A2B10G10R10 = 18,
	PF_A16B16G16R16 = 19,
	PF_D24 = 20,
	PF_R16F = 21,
	PF_R16F_FILTER = 22,
	PF_BC5 = 23,
	PF_V8U8 = 24,
	PF_A1 = 25,
	PF_FloatR11G11B10 = 26,
	PF_A8 = 27,
	PF_R32_UINT = 28,
	PF_R32_SINT = 29,
	PF_PVRTC2 = 30,
	PF_PVRTC4 = 31,
	PF_R16_UINT = 32,
	PF_R16_SINT = 33,
	PF_R16G16B16A16_UINT = 34,
	PF_R16G16B16A16_SINT = 35,
	PF_R5G6B5_UNORM = 36,
	PF_R8G8B8A8 = 37,
	PF_A8R8G8B8 = 38,	// Only used for legacy loading; do NOT use!
	PF_BC4 = 39,
	PF_R8G8 = 40,
	PF_ATC_RGB = 41,	// Unsupported Format
	PF_ATC_RGBA_E = 42,	// Unsupported Format
	PF_ATC_RGBA_I = 43,	// Unsupported Format
	PF_X24_G8 = 44,	// Used for creating SRVs to alias a DepthStencil buffer to read Stencil. Don't use for creating textures.
	PF_ETC1 = 45,	// Unsupported Format
	PF_ETC2_RGB = 46,
	PF_ETC2_RGBA = 47,
	PF_R32G32B32A32_UINT = 48,
	PF_R16G16_UINT = 49,
	PF_ASTC_4x4 = 50,	// 8.00 bpp
	PF_ASTC_6x6 = 51,	// 3.56 bpp
	PF_ASTC_8x8 = 52,	// 2.00 bpp
	PF_ASTC_10x10 = 53,	// 1.28 bpp
	PF_ASTC_12x12 = 54,	// 0.89 bpp
	PF_BC6H = 55,
	PF_BC7 = 56,
	PF_R8_UINT = 57,
	PF_L8 = 58,
	PF_XGXR8 = 59,
	PF_R8G8B8A8_UINT = 60,
	PF_R8G8B8A8_SNORM = 61,
	PF_R16G16B16A16_UNORM = 62,
	PF_R16G16B16A16_SNORM = 63,
	PF_PLATFORM_HDR_0 = 64,
	PF_PLATFORM_HDR_1 = 65,	// Reserved.
	PF_PLATFORM_HDR_2 = 66,	// Reserved.
	PF_NV12 = 67,
	PF_R32G32_UINT = 68,
	PF_ETC2_R11_EAC = 69,
	PF_ETC2_RG11_EAC = 70,
	PF_R8 = 71,
	PF_B5G5R5A1_UNORM = 72,
	PF_ASTC_4x4_HDR = 73,
	PF_ASTC_6x6_HDR = 74,
	PF_ASTC_8x8_HDR = 75,
	PF_ASTC_10x10_HDR = 76,
	PF_ASTC_12x12_HDR = 77,
	PF_G16R16_SNORM = 78,
	PF_R8G8_UINT = 79,
	PF_R32G32B32_UINT = 80,
	PF_R32G32B32_SINT = 81,
	PF_R32G32B32F = 82,
	PF_R8_SINT = 83,
	PF_R64_UINT = 84,
	PF_R9G9B9EXP5 = 85,
	PF_P010 = 86,
};
#endif
