// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/UnrealEngine/TextureShareSDKUnrealEngineTypes.h"

#ifndef __UNREAL__
/**
 * Implementation of UnrealEngine containers on the SDK side
 */


/**
 * Reflect UE funcs to SDK ext app side
 * The code below is copied from the UE
 */
struct FMath
{
	/** Computes absolute value in a generic way */
	template< class T >
	static constexpr inline T Abs(const T A)
	{
		return (A >= (T)0) ? A : -A;
	}

	/** Returns 1, 0, or -1 depending on relation of T to 0 */
	template< class T >
	static constexpr inline T Sign(const T A)
	{
		return (A > (T)0) ? (T)1 : ((A < (T)0) ? (T)-1 : (T)0);
	}

	/** Returns higher value in a generic way */
	template< class T >
	static constexpr inline T Max(const T A, const T B)
	{
		return (A >= B) ? A : B;
	}

	/** Returns lower value in a generic way */
	template< class T >
	static constexpr inline T Min(const T A, const T B)
	{
		return (A <= B) ? A : B;
	}

	// Allow mixing of float types to promote to highest precision type
	static constexpr inline double Max(const double A, const float B) { return Max<double>(A, B); }
	static constexpr inline double Max(const float A, const double B) { return Max<double>(A, B); }
	static constexpr inline double Min(const double A, const float B) { return Min<double>(A, B); }
	static constexpr inline double Min(const float A, const double B) { return Min<double>(A, B); }

};

#endif
