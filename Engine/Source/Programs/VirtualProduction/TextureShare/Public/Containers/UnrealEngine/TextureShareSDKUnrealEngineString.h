// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/UnrealEngine/TextureShareSDKUnrealEngineTypes.h"

#ifndef __UNREAL__
/**
 * Implementation of UnrealEngine FString on the SDK side
 * The code below is copied from the UE
 */

#include <xstring>
#include <tchar.h>

//@todo: implement better logic for std::wstring vs FString (this is too simplistic now)

/**
 * String support for SDK ext app side
 */
namespace ESearchCase
{
	enum Type
	{
		CaseSensitive,
		IgnoreCase,
	};
}

/**
 * String support for SDK ext app side
 */
namespace FPlatformString
{
	/**
	 * Type trait which yields an unsigned integer type of a given number of bytes.
	 * If there is no such type, the Type member type will be absent, allowing it to be used in SFINAE contexts.
	 */
	template <int NumBytes>
	struct TUnsignedIntType
	{ };

	template <> struct TUnsignedIntType<1> { using Type = uint8; };
	template <> struct TUnsignedIntType<2> { using Type = uint16; };
	template <> struct TUnsignedIntType<4> { using Type = uint32; };
	template <> struct TUnsignedIntType<8> { using Type = uint64; };

	/**
	* Avoid sign extension problems with signed characters smaller than int
	*
	* E.g. 'Ö' - 'A' is negative since the char 'Ö' (0xD6) is negative and gets
	* sign-extended to the 32-bit int 0xFFFFFFD6 before subtraction happens.
	*
	* Mainly needed for subtraction and addition.
	*/
	template <typename CharType>
	struct TChar
	{
		static constexpr uint32 ToUnsigned(CharType Char)
		{
			return (typename TUnsignedIntType<sizeof(CharType)>::Type)Char;
		}
	};

	typedef wchar_t WIDECHAR;
	typedef size_t SIZE_T;

	// Copied from \\Engine\Source\Runtime\Core\Private\GenericPlatform\GenericPlatformStricmp.cpp
	static constexpr uint8 LowerAscii[128] = {
		0x00, 0x01, 0x02, 0x03, 0x04, 0x05, 0x06, 0x07, 0x08, 0x09, 0x0A, 0x0B, 0x0C, 0x0D, 0x0E, 0x0F,
		0x10, 0x11, 0x12, 0x13, 0x14, 0x15, 0x16, 0x17, 0x18, 0x19, 0x1A, 0x1B, 0x1C, 0x1D, 0x1E, 0x1F,
		0x20, 0x21, 0x22, 0x23, 0x24, 0x25, 0x26, 0x27, 0x28, 0x29, 0x2A, 0x2B, 0x2C, 0x2D, 0x2E, 0x2F,
		0x30, 0x31, 0x32, 0x33, 0x34, 0x35, 0x36, 0x37, 0x38, 0x39, 0x3A, 0x3B, 0x3C, 0x3D, 0x3E, 0x3F,
		0x40, 'a',  'b',  'c',  'd',  'e',  'f',  'g',  'h',  'i',  'j',  'k',  'l',  'm',  'n',  'o',
		'p',  'q',  'r',  's',  't',  'u',  'v',  'w',  'x',  'y',  'z',  0x5B, 0x5C, 0x5D, 0x5E, 0x5F,
		0x60, 0x61, 0x62, 0x63, 0x64, 0x65, 0x66, 0x67, 0x68, 0x69, 0x6A, 0x6B, 0x6C, 0x6D, 0x6E, 0x6F,
		0x70, 0x71, 0x72, 0x73, 0x74, 0x75, 0x76, 0x77, 0x78, 0x79, 0x7A, 0x7B, 0x7C, 0x7D, 0x7E, 0x7F
	};

	inline bool BothAscii(uint32 C1, uint32 C2)
	{
		return ((C1 | C2) & 0xffffff80) == 0;
	}

	inline int32 Strcmp(const WIDECHAR* String1, const WIDECHAR* String2)
	{
		return (int32)_tcscmp(String1, String2);
	}

	template<typename CharType1, typename CharType2>
	inline int32 StricmpImpl(const CharType1* String1, const CharType2* String2)
	{
		while (true)
		{
			CharType1 C1 = *String1++;
			CharType2 C2 = *String2++;

			uint32 U1 = TChar<CharType1>::ToUnsigned(C1);
			uint32 U2 = TChar<CharType2>::ToUnsigned(C2);

			// Quickly move on if characters are identical but
			// return equals if we found two null terminators
			if (U1 == U2)
			{
				if (U1)
				{
					continue;
				}

				return 0;
			}
			else if (BothAscii(U1, U2))
			{
				if (int32 Diff = LowerAscii[U1] - LowerAscii[U2])
				{
					return Diff;
				}
			}
			else
			{
				return U1 - U2;
			}
		}
	}

	template<typename CharType1, typename CharType2>
	inline int32 StrnicmpImpl(const CharType1* String1, const CharType2* String2, SIZE_T Count)
	{
		for (; Count > 0; --Count)
		{
			CharType1 C1 = *String1++;
			CharType2 C2 = *String2++;

			// Quickly move on if characters are identical but
			// return equals if we found two null terminators
			if (C1 == C2)
			{
				if (C1)
				{
					continue;
				}

				return 0;
			}
			else if (BothAscii(C1, C2))
			{
				if (int32 Diff = LowerAscii[TChar<CharType1>::ToUnsigned(C1)] - LowerAscii[TChar<CharType2>::ToUnsigned(C2)])
				{
					return Diff;
				}
			}
			else
			{
				return TChar<CharType1>::ToUnsigned(C1) - TChar<CharType2>::ToUnsigned(C2);
			}
		}

		return 0;
	}

	inline int32 Stricmp(const WIDECHAR* Str1, const WIDECHAR* Str2) { return StricmpImpl(Str1, Str2); }
};

// Implement FString simple container for SDK data type mirroring
struct FString
{
	std::wstring  Data;

	typedef wchar_t CharType;
	typedef int32   SizeType;

public:
	FString() = default;

	FString(const FString& In)
	{
		Data = *In;
	}

	FString(const CharType* In)
	{
		Data = In;
	}

	FString(const CharType* In1, const CharType* In2)
	{
		Data = In1;
		Data += In2;
	}

	FString(const CharType* In1, const CharType* In2, const CharType* In3)
	{
		Data = In1;
		Data += In2;
		Data += In3;
	}

	FString(const CharType* In1, const CharType* In2, const CharType* In3, const CharType* In4)
	{
		Data = In1;
		Data += In2;
		Data += In3;
		Data += In4;
	}

public:
	inline void operator=(const FString& In)
	{
		Data = *In;
	}

	inline void operator=(const CharType* In)
	{
		Data = In;
	}

	inline const CharType* operator*() const
	{
		return Data.c_str();
	}

	inline SizeType Len() const
	{
		return (SizeType)Data.length();
	}

	inline bool IsEmpty() const
	{
		return Data.empty();
	}

public:
	/**
	 * Lexicographically tests whether this string is equivalent to the Other given string
	 *
	 * @param Other 	The string test against
	 * @param SearchCase 	Whether or not the comparison should ignore case
	 * @return true if this string is lexicographically equivalent to the other, otherwise false
	 */
	inline bool Equals(const FString& Other, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive) const
	{
		SizeType Num = Len();
		SizeType OtherNum = Other.Len();

		if (Num != OtherNum)
		{
			// Handle special case where FString() == FString("")
			return Num + OtherNum == 1;
		}
		else if (Num > 1)
		{
			if (SearchCase == ESearchCase::CaseSensitive)
			{
				return FPlatformString::Strcmp(Data.c_str(), Other.Data.c_str()) == 0;
			}
			else
			{
				return FPlatformString::Stricmp(Data.c_str(), Other.Data.c_str()) == 0;
			}
		}

		return true;
	}

	/**
	 * Lexicographically test whether the left string is == the right string
	 *
	 * @param Lhs String to compare against.
	 * @param Rhs String to compare against.
	 * @return true if the left string is lexicographically == the right string, otherwise false
	 * @note case insensitive
	 */
	friend inline bool operator==(const FString& Lhs, const FString& Rhs)
	{
		return Lhs.Equals(Rhs, ESearchCase::IgnoreCase);
	}

	/**
	 * Lexicographically test whether the left string is == the right string
	 *
	 * @param Lhs String to compare against.
	 * @param Rhs String to compare against.
	 * @return true if the left string is lexicographically == the right string, otherwise false
	 * @note case insensitive
	 */
	friend inline bool operator==(const FString& Lhs, const CharType* Rhs)
	{
		return FPlatformString::Stricmp(*Lhs, Rhs) == 0;
	}
};

#endif
