// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef __UNREAL__
/**
 * Implementation of atomic UE types on the SDK side
 * The code below is copied from the UE
 */

namespace Windows
{
	typedef void* HANDLE;
};

// Standard typedefs
typedef signed char         schar;
typedef signed char         int8;
typedef short               int16;
typedef int                 int32;
#ifdef _MSC_VER
typedef __int64             int64;
#else
typedef long long           int64;
#endif /* _MSC_VER */

typedef unsigned char      uint8;
typedef unsigned short     uint16;
typedef unsigned int       uint32;
#ifdef _MSC_VER
typedef unsigned __int64   uint64;
#else
typedef unsigned long long uint64;
#endif /* _MSC_VER */

#endif
