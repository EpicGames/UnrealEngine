// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShareSDKObject.h"

/**
 * TextureShareSDK API
 */
class ITextureShareSDK
{
public:
	/**
	* Get exist or create new ITextureShareSDKObject object
	*
	* @param ShareName   - Unique share name
	*
	* @return - Pointer to an existing/new interface, or null (if the name is already in use in another interface type)
	*/
	TEXTURESHARESDK_API static ITextureShareSDKObject* GetOrCreateObject(const wchar_t* ShareName);

	/**
	 * Check if ITextureShareSDKObject object with that name exist
	 *
	 * @param ShareName   - Unique share name
	 *
	 * @return - true if exist
	 */
	TEXTURESHARESDK_API static  bool IsObjectExist(const wchar_t* ShareName);

	/**
	 * Release ITextureShareSDKObject object
	 *
	 * @param ShareName   - Unique share name
	 *
	 * @return - true if released
	 */
	TEXTURESHARESDK_API static  bool RemoveObject(const wchar_t* ShareName);

	/**
	 * Returns a description of all objects
	 *
	 * @param OutInterprocessObjects - All exist texture share objects on this computer
	 * @param ShareName              - (optional) return only objects with this name
	 *
	 * @return - true if success
	 */
	TEXTURESHARESDK_API static bool GetInterprocessObjects(TDataOutput<TArraySerializable<FTextureShareCoreObjectDesc>>& OutInterprocessObjects, const wchar_t* ShareName = nullptr);

	/**
	 * Returns detailed information about the local process
	 */
	TEXTURESHARESDK_API static void GetProcessDesc(TDataOutput<FTextureShareCoreObjectProcessDesc>& OutProcessDesc);

	/**
	 * Assign a unique name to the local process. The changes only affect newly created TextureShare objects.
	 *
	 * @param InProcessId - Custom local process name
	 */
	TEXTURESHARESDK_API static void SetProcessName(const wchar_t* InProcessId);

	/**
	 * Assign a device type to the local process. The changes only affect newly created TextureShare objects.
	 *
	 * @param InDeviceType - render device type
	 */
	TEXTURESHARESDK_API static bool SetProcessDeviceType(const ETextureShareDeviceType InDeviceType);

	/**
	 * Release all resources last used more than specified timeout value
	 *
	 * @param InMilisecondsTimeOut - Timeout to remove resource
	 */
	TEXTURESHARESDK_API static void RemoveUnusedResources(const uint32 InMilisecondsTimeOut);
};
