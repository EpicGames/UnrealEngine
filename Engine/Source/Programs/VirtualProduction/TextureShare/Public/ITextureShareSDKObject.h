// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/TextureShareSDKContainers.h"
#include "Containers/TextureShareCoreContainers.h"
#include "Serialize/TextureShareSDKDataWrappers.h"

/**
 * TextureShareSDK object API
 */
class TEXTURESHARESDK_API ITextureShareSDKObject
{
public:
	virtual ~ITextureShareSDKObject() = default;

public:
	///////////////////////// State /////////////////////////

	/**
	* Return the name of the TextureShare.
	*/
	virtual const wchar_t* GetName() const = 0;

	/**
	* Returns detailed information about the TextureShare object.
	*/
	virtual void GetObjectDesc(TDataOutput<FTextureShareCoreObjectDesc>& OutObjectDesc) const = 0;

	/**
	* Returns detailed information about the TextureShare proxy object.
	*/
	virtual void GetObjectDesc_RenderThread(TDataOutput<FTextureShareCoreObjectDesc>& OutObjectDesc) const = 0;

	/**
	 * Returns true if the TextureShare object is ready to be used.
	 */
	virtual bool IsActive() const = 0;

	/**
	 * Returns true if the TextureShare object proxy is ready to be used.
	 */
	virtual bool IsActive_RenderThread() const = 0;

	/**
	 * Returns true if the TextureShareCore object has started a session and processes are connected for this frame.
	 */
	virtual bool IsFrameSyncActive() const = 0;

	/**
	 * Returns true if the TextureShareCore object proxy has started a session and processes are connected for this frame.
	 */
	virtual bool IsFrameSyncActive_RenderThread() const = 0;

	/**
	 * Support for multi-threaded implementation.
	 * Returns true if the BeginFrameSync() function can be called at this moment for object
	 */
	virtual bool IsBeginFrameSyncActive() const = 0;

	/**
	 * Support for multi-threaded implementation.
	 * Returns true if the BeginFrameSync() function can be called at this moment for object proxy
	 */
	virtual bool IsBeginFrameSyncActive_RenderThread() const = 0;

public:
	///////////////////////// Settings /////////////////////////

	/**
	 * Change the process name for this TextureShare object.
	 * If the object is synced now, then the changes are deferred and executed in the next frame.
	 *
	 * @param InProcessId - New process name
	 *
	 * @return true, if success
	 */
	virtual bool SetProcessId(const wchar_t* InProcessId) = 0;

	/**
	 * Assign a device type for this TextureShare object.
	 *
	 * @param InDeviceType - render device type
	 */
	virtual bool SetDeviceType(const ETextureShareDeviceType InDeviceType) = 0;

	/**
	 * Change the sync settings for this TextureShare object.
	 * If the object is synced now, then the changes are deferred and executed in the next frame.
	 *
	 * @param InSyncSetting - New sync settings
	 *
	 * @return true, if success
	 */
	virtual bool SetSyncSetting(const TDataInput<FTextureShareCoreSyncSettings>& InSyncSetting) = 0;

	/**
	* Return the sync settings of the TextureShare.
	*/
	virtual void GetSyncSetting(TDataOutput<FTextureShareCoreSyncSettings>& OutSyncSettings) const = 0;

	/**
	 * Get TextureShare default sync settings by template type
	 * The settings of this template are not related to the current settings of the object
	 *
	 * @param InType - Sync settings template type
	 *
	 * @return structure with sync logic settings
	 */
	virtual void GetFrameSyncSettings(const ETextureShareFrameSyncTemplate InType, TDataOutput<FTextureShareCoreFrameSyncSettings>& OutFrameSyncSettings) const = 0;

	/**
	 * Find skipped frame sync step
	 *
	 * @param InSyncStep           - Sync step value
	 * @param OutSkippedSyncStep   - Skipped sync step value
	 *
	 * @return true, if skipped sync step found
	 */
	virtual bool FindSkippedSyncStep(const ETextureShareSyncStep InSyncStep, ETextureShareSyncStep& OutSkippedSyncStep) const = 0;

	/**
	 * Find skipped frame sync step for proxy object
	 *
	 * @param InSyncStep           - Sync step value
	 * @param OutSkippedSyncStep   - Skipped sync step value
	 *
	 * @return true, if skipped sync step found
	 */
	virtual bool FindSkippedSyncStep_RenderThread(const ETextureShareSyncStep InSyncStep, ETextureShareSyncStep& OutSkippedSyncStep) const = 0;

public:
	///////////////////////// Session /////////////////////////

	/**
	 * Start a TextureShare session for the specified device
	 *
	 * @param InDeviceType     - Shared resources render device type
	 *
	 * @return True if the success
	 */
	virtual bool BeginSession() = 0;

	/**
	 * End the TextureShare session and the resources in use are also removed.
	 *
	 * @return True if the success
	 */
	virtual bool EndSession() = 0;

	/**
	 * Returns true if the session is currently valid
	 */
	virtual bool IsSessionActive() const = 0;

public:
	///////////////////////// Thread sync support /////////////////////////

	/**
	 * Support for multi-threaded implementation.
	 * The mutex of the specified type will be locked (and created for the first time)
	 *
	 * @param InThreadMutex - Mutex type
	 * @param bForceLockNoWait - lock without waiting for unlock
	 *
	 * @return True if the success
	 */
	virtual bool LockThreadMutex(const ETextureShareThreadMutex InThreadMutex, bool bForceLockNoWait = false) = 0;

	/**
	 * Support for multi-threaded implementation.
	 * The mutex of the specified type will be unlocked
	 *
	 * @param InThreadMutex - Mutex type
	 *
	 * @return True if the success
	 */
	virtual bool UnlockThreadMutex(const ETextureShareThreadMutex InThreadMutex) = 0;

public:
	///////////////////////// Interprocess Synchronization /////////////////////////

	/**
	 * Begin sync logic in range FrameBegin..FrameEnd
	 * The list of connected processes for the current frame will also be updated.
	 * Returns true if the frame is connected to other processes that match the synchronization settings.
	 */
	virtual bool BeginFrameSync() = 0;

	/**
	 * Synchronize connected processes that support this sync step
	 * Data from remote processes will be read at the time the barrier is synchronized.
	 * Missed sync steps from the sync settings will also be called.
	 *
	 * @param InSyncStep - Sync step value
	 *
	 * @return True if the success
	 */
	virtual bool FrameSync(const ETextureShareSyncStep InSyncStep) = 0;

	/**
	 * Finalize sync logic in range FrameBegin..FrameEnd
	 * Missed sync steps from the sync settings will also be called.
	 */
	virtual bool EndFrameSync() = 0;

	/**
	 * Begin sync logic in range FrameProxyBegin..FrameProxyEnd
	 */
	virtual bool BeginFrameSync_RenderThread() = 0;

	/**
	 * Synchronize connected processes that support this sync step
	 * ProxyData from remote processes will be read at the time the barrier is synchronized.
	 * Missed sync steps from the sync settings will also be called.
	 *
	 * @param InSyncStep - Sync step value
	 *
	 * @return True if the success
	 */
	virtual bool FrameSync_RenderThread(const ETextureShareSyncStep InSyncStep) = 0;

	/**
	 * Finalize sync logic in range FrameProxyBegin..FrameProxyEnd
	 * Missed sync steps from the sync settings will also be called.
	 */
	virtual bool EndFrameSync_RenderThread() = 0;

	/**
	 * Returns a list of handles to the interprocess TextureShare objects that are currently connected to this object.
	 *
	 * @param OutInterprocessObjects - Output array with handles to TextureShare objects
	 *
	 * @return true, if success
	 */
	virtual void GetConnectedInterprocessObjects(TDataOutput<TArraySerializable<FTextureShareCoreObjectDesc>>& OutObjectList) const = 0;

public:
	///////////////////////// Data Containers /////////////////////////

	/**
	 * Get object data
	 * Object data for the current frame in the GameThread.
	 */
	virtual void GetData(TDataOutput<FTextureShareCoreData>& OutObjectData) = 0;

	/**
	 * Set object data
	 * Set new object data for the current frame in the GameThread.
	 */
	virtual void SetData(const TDataInput<FTextureShareCoreData>& InObjectData) = 0;

	/**
	 * Get object ProxyData.
	 * Object proxy data for the current frame in the RenderThread.
	 */
	virtual void GetProxyData_RenderThread(TDataOutput<FTextureShareCoreProxyData>& OutObjectProxyData) = 0;

	/**
	 * Set object ProxyData.
	 * Object proxy data for the current frame in the RenderThread.
	 */
	virtual void SetProxyData_RenderThread(const TDataInput<FTextureShareCoreProxyData>& InObjectProxyData) = 0;

	/**
	 * Received Data from connected process objects
	 */
	virtual void GetReceivedData(TDataOutput<TArraySerializable<FTextureShareCoreObjectData>>& OutReceivedObjectsData) const = 0;

	/**
	 * Received ProxyData from connected process objects
	 */
	virtual void GetReceivedProxyData_RenderThread(TDataOutput<TArraySerializable<FTextureShareCoreObjectProxyData>>& OutObjectsProxyData) const = 0;

public:
	///////////////////////// Access to shared resources /////////////////////////

	/**
	 * Open shared D3D11 resource
	 *
	 * @param InD3D11Device - D3D11 device interface
	 * @param InResourceDesc- Shared resource descriptor
	 *
	 * @return pointer to shared D3D11 texture, or nullptr on failure
	 */
	virtual ID3D11Texture2D* OpenSharedResourceD3D11(ID3D11Device* InD3D11Device, const TDataInput<FTextureShareCoreResourceDesc>& InResourceDesc) = 0;

	/**
	 * Open shared D3D12 resource
	 *
	 * @param InD3D12Device - D3D12 device interface
	 * @param InResourceDesc- Shared resource descriptor
	 *
	 * @return pointer to shared D3D12 texture, or nullptr on failure
	 */
	virtual  ID3D12Resource* OpenSharedResourceD3D12(ID3D12Device* InD3D12Device, const TDataInput<FTextureShareCoreResourceDesc>& InResourceDesc) = 0;

	/**
	 * Open shared Vulkan resource
	 *
	 * @parama InDeviceVulkanContext - Vulkan device context
	 * @param InResourceDesc         - Shared resource descriptor
	 * @param OutVulkanResource      - Output Vulkan resource
	 *
	 * @return true, if resource opened
	 */
	virtual  bool OpenSharedResourceVulkan(const TDataInput<FTextureShareDeviceVulkanContext>& InDeviceVulkanContext, const TDataInput<FTextureShareCoreResourceDesc>& InResourceDesc, TDataOutput<FTextureShareDeviceVulkanResource>& OutVulkanResource) = 0;
};
