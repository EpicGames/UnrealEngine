// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/TextureShareSDKContainers.h"
#include "Serialize/ITextureShareSerialize.h"

#ifndef __UNREAL__

/**
 * Support SDK data types serialization
 */
class ITextureShareSDKSerializeStream
	: public ITextureShareSerializeStream
{
public:
	virtual ~ITextureShareSDKSerializeStream() = default;

	// Serialize any serializable objects
	virtual ITextureShareSerializeStream& operator<<(ITextureShareSerialize& InOutData) override
	{
		return InOutData.Serialize(*this);
	}

public:
	// Serialize atomic types:
	virtual ITextureShareSerializeStream& operator<<(int8& In)  override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(int16& In) override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(int32& In) override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(int64& In) override { return this->SerializeData(&In, sizeof(In)); }

	virtual ITextureShareSerializeStream& operator<<(uint8& In)  override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(uint16& In) override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(uint32& In) override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(uint64& In) override { return this->SerializeData(&In, sizeof(In)); }

	virtual ITextureShareSerializeStream& operator<<(float& In)  override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(double& In) override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(bool& In)   override { return this->SerializeData(&In, sizeof(In)); }

public:
	// Serialize TextureShare enums:
	virtual ITextureShareSerializeStream& operator<<(ETextureShareFrameSyncTemplate& In) override { return this->SerializeData(&In, sizeof(In)); }

	virtual ITextureShareSerializeStream& operator<<(ETextureShareSyncPass& In)     override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(ETextureShareSyncState& In)    override { return this->SerializeData(&In, sizeof(In)); }
	
	virtual ITextureShareSerializeStream& operator<<(ETextureShareSyncStep& In)     override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(ETextureShareResourceType& In) override { return this->SerializeData(&In, sizeof(In)); }

	virtual ITextureShareSerializeStream& operator<<(ETextureShareTextureOp& In)   override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(ETextureShareEyeType& In)     override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(ETextureShareDeviceType& In)  override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(ETextureShareProcessType& In) override { return this->SerializeData(&In, sizeof(In)); }

	virtual ITextureShareSerializeStream& operator<<(ETextureShareCoreSceneViewManualProjectionType& In)   override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(ETextureShareViewRotationDataType& In) override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(ETextureShareViewLocationDataType& In) override { return this->SerializeData(&In, sizeof(In)); }

public:
	// Serialize other enums
	virtual ITextureShareSerializeStream& operator<<(EPixelFormat& In) override { return this->SerializeData(&In, sizeof(In)); }
	virtual ITextureShareSerializeStream& operator<<(DXGI_FORMAT& In)  override { return this->SerializeData(&In, sizeof(In)); }

public:
	// Serialize UECore types:
	virtual ITextureShareSerializeStream& operator<<(FString& InOutData) override
	{
		int32 StrLen = InOutData.Len();
		*this << StrLen;

		if (StrLen > 0)
		{
			if (IsWriteStream())
			{
				const wchar_t* WCharValue = *InOutData;
				SerializeData(WCharValue, sizeof(wchar_t) * StrLen);
			}
			else
			{
				TArray<wchar_t> WCharBuffer;
				WCharBuffer.AddZeroed(StrLen + 1);
				SerializeData(WCharBuffer.GetData(), sizeof(wchar_t) * StrLen);
				const wchar_t* WCharValue = WCharBuffer.GetData();

				InOutData = WCharValue;
			}
		}

		return *this;
	}

	virtual ITextureShareSerializeStream& operator<<(FVector& InOutData) override
	{
		return (*this) << InOutData.X << InOutData.Y << InOutData.Z;
	}

	virtual ITextureShareSerializeStream& operator<<(FVector2D& InOutData) override
	{
		return (*this) << InOutData.X << InOutData.Y;
	}

	virtual ITextureShareSerializeStream& operator<<(FRotator& InOutData) override
	{
		return (*this) << InOutData.Pitch << InOutData.Yaw << InOutData.Roll;
	}

	virtual ITextureShareSerializeStream& operator<<(FQuat& InOutData) override
	{
		return (*this) << InOutData.X << InOutData.Y << InOutData.Z << InOutData.W;
	}

	virtual ITextureShareSerializeStream& operator<<(FMatrix& InOutData) override
	{
		return (*this)
			<< InOutData.M[0][0] << InOutData.M[0][1] << InOutData.M[0][2] << InOutData.M[0][3]
			<< InOutData.M[1][0] << InOutData.M[1][1] << InOutData.M[1][2] << InOutData.M[1][3]
			<< InOutData.M[2][0] << InOutData.M[2][1] << InOutData.M[2][2] << InOutData.M[2][3]
			<< InOutData.M[3][0] << InOutData.M[3][1] << InOutData.M[3][2] << InOutData.M[3][3];
	}

	virtual ITextureShareSerializeStream& operator<<(FIntRect& InOutData) override
	{
		return (*this) << InOutData.Min << InOutData.Max;
	}

	virtual ITextureShareSerializeStream& operator<<(FIntPoint& InOutData) override
	{
		return (*this) << InOutData.X << InOutData.Y;
	}

	virtual ITextureShareSerializeStream& operator<<(FGuid& InOutData) override
	{
		return (*this) << InOutData.A << InOutData.B << InOutData.C << InOutData.D;
	}

	virtual ITextureShareSerializeStream& operator<<(Windows::HANDLE& InOutData) override
	{
		return SerializeData(&InOutData, sizeof(Windows::HANDLE));
	}
};

#endif
