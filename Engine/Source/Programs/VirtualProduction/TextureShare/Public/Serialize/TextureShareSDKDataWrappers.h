// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/TextureShareSDKContainers.h"

#if __UNREAL__
// implement binary serialization on the Unreal Engine side
#include "Core/Serialize/ITextureShareCoreSerializeStream.h"
using IDataContainerSerializeStream = ITextureShareCoreSerializeStream;

#else
// implement binary serialization on the user side
#include "Serialize/ITextureShareSDKSerializeStream.h"
using IDataContainerSerializeStream = ITextureShareSDKSerializeStream;

/**
 * Copied from UE for SDK ext app side
 */
namespace FPlatformMemory
{
	typedef size_t SIZE_T;

	inline void* Memcpy(void* Dest, const void* Src, SIZE_T Count)
	{
		return memcpy(Dest, Src, Count);
	}
};

#define check(Cond)
#endif

namespace DataSerializer
{
	enum Limits
	{
		// Transaction memory block size
		MaxStreamDataSize = (32 * 1024),
	};

	/**
	 * Binary serializer (Write)
	 */
	struct FStreamWrite
		: public IDataContainerSerializeStream
	{
		uint8* const DataPtr = nullptr;
		const uint32_t DataSize = 0;

		uint32_t CurrentPos = 0;

	public:
		FStreamWrite(uint8* const InDataPtr, const uint32_t InDataSize)
			: DataPtr(InDataPtr), DataSize(InDataSize)
		{ }

		virtual ~FStreamWrite() = default;

		virtual bool IsWriteStream() const
		{
			return true;
		}

	protected:
		virtual FStreamWrite& SerializeData(void* InDataPtr, const uint32_t InDataSize) override
		{
			return ImplWriteData(InDataPtr, InDataSize);
		}

		virtual FStreamWrite& SerializeData(const void* InDataPtr, const uint32_t InDataSize) override
		{
			return ImplWriteData(InDataPtr, InDataSize);
		}

	private:
		FStreamWrite& ImplWriteData(const void* InDataPtr, const uint32_t InDataSize)
		{
			const uint32_t NextPos = CurrentPos + InDataSize;

			check(NextPos <= DataSize);

			// Copy data to the shared memory
			FPlatformMemory::Memcpy(DataPtr + CurrentPos, InDataPtr, InDataSize);

			// udate current writing pos
			CurrentPos = NextPos;

			return *this;
		}
	};

	/**
	 * Binary serializer (Read)
	 */
	class FStreamRead
		: public IDataContainerSerializeStream
	{
		const uint8*   DataPtr = nullptr;
		const uint32_t DataSize = 0;

		uint32_t CurrentPos = 0;

	public:
		FStreamRead(const uint8* InDataPtr, const uint32_t InDataSize)
			: DataPtr(InDataPtr), DataSize(InDataSize)
		{ }

		virtual ~FStreamRead() = default;

		virtual bool IsWriteStream() const
		{
			return false;
		}

	protected:
		virtual FStreamRead& SerializeData(void* InDataPtr, const uint32_t InDataSize) override
		{
			return ImplReadData(InDataPtr, InDataSize);
		}

	private:
		FStreamRead& ImplReadData(void* InDataPtr, const uint32_t InDataSize)
		{
			if (InDataPtr && DataSize)
			{
				// Implement read
				const uint32_t NextPos = CurrentPos + InDataSize;

				// check vs structure size diffs
				check(NextPos <= DataSize);

				FPlatformMemory::Memcpy(InDataPtr, DataPtr + CurrentPos, InDataSize);

				// Update current reading pos
				CurrentPos = NextPos;
			}

			return *this;
		}
	};
};

/**
 * Data wrapper SDK.DLL->ExtApp
 */
template<typename T>
struct TDataOutput
{
	// Store structures as binary. And exchange like this
	// Because the DLL versus the user application have differences in compilation, library version, memory alignment, etc.
	uint32_t DataSize = 0;
	uint8 DataPtr[DataSerializer::MaxStreamDataSize];
public:
	TDataOutput(T& InOutData)
		: OutDataRef(InOutData)
	{ }

	~TDataOutput()
	{
		// Deserialize data
		DataSerializer::FStreamRead SerializedDataArchive(DataPtr, DataSize);
		SerializedDataArchive << OutDataRef;
	}

	TDataOutput<T>& operator=(const T& InData)
	{
		T Data(InData);
		DataSerializer::FStreamWrite SerializedDataArchive(DataPtr, DataSerializer::MaxStreamDataSize);
		SerializedDataArchive << Data;
		DataSize = SerializedDataArchive.CurrentPos;

		return *this;
	}

	TDataOutput<T>& operator=(T& InData)
	{
		DataSerializer::FStreamWrite SerializedDataArchive(DataPtr, DataSerializer::MaxStreamDataSize);
		SerializedDataArchive << InData;
		DataSize = SerializedDataArchive.CurrentPos;

		return *this;
	}

	TDataOutput<T>& operator*()
	{
		return *this;
	}

private:
	T& OutDataRef;
};

/**
 * Data wrapper ExtApp->SDK.DLL
 */
template<typename T>
struct TDataInput
{
	// Store structures as binary. And exchange like this
	// Because the DLL versus the user application have differences in compilation, library version, memory alignment, etc.
	uint8 DataPtr[DataSerializer::MaxStreamDataSize];
	uint32_t DataSize = 0;

public:
	TDataInput(const T& InData)
	{
		T Data(InData);
		DataSerializer::FStreamWrite SerializedDataArchive(DataPtr, DataSerializer::MaxStreamDataSize);
		SerializedDataArchive << Data;
		DataSize = SerializedDataArchive.CurrentPos;
	}

	TDataInput(T& InData)
	{
		DataSerializer::FStreamWrite SerializedDataArchive(DataPtr, DataSerializer::MaxStreamDataSize);
		SerializedDataArchive << InData;
		DataSize = SerializedDataArchive.CurrentPos;
	}

	inline T& Deserialize(T& OutData) const
	{
		// Deserialize data
		DataSerializer::FStreamRead SerializedDataArchive(DataPtr, DataSize);
		SerializedDataArchive << OutData;

		return OutData;
	}
};

/**
 * Data deserializer TDataInput<InType> to InType
 */
template<typename T>
struct TDataVariable
{
	T OutData;

	TDataVariable(const TDataInput<T>& InData)
	{
		InData.Deserialize(OutData);
	}


	T& operator*()
	{
		return OutData;
	}

	const T& operator*() const
	{
		return OutData;
	}
};
