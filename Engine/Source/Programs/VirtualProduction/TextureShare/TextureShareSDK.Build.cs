// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
namespace UnrealBuildTool.Rules
{
	[SupportedPlatforms("Win64")]
	public class TextureShareSDK : ModuleRules
	{
		public TextureShareSDK(ReadOnlyTargetRules Target)
			: base(Target)
		{
			bRequiresImplementModule = false;
			PrivatePCHHeaderFile = "Public/ITextureShareSDK.h";

			PrivateIncludePaths.AddRange(
				new string[] {
					Path.Combine(EngineDirectory,"Plugins/VirtualProduction/TextureShare/Source/TextureShareCore/Private"),
				}
			);

			PrivateDependencyModuleNames.AddRange( 
				new string[]
				{
					"Core",
					"ImageCore",
					"TextureShareCore"
				}
			);
		}
	}
}
