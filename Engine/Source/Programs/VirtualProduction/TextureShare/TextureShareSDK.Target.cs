// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;
using System;
using System.IO;
using EpicGames.Core;
using System.Runtime.CompilerServices;

[SupportedPlatforms("Win64")]
public class TextureShareSDKTarget : TargetRules
{
	private void ImplPostBuildCopy(string SrcPath, string DestPath)
	{
		PostBuildSteps.Add(string.Format("echo Copying {0} to {1}", SrcPath, DestPath));
		PostBuildSteps.Add(string.Format("xcopy /y /i /v \"{0}\" \"{1}\" 1>nul", SrcPath, DestPath));
	}

	private void AddWindowsPostBuildSteps()
	{
		string SDKDestPath = Path.Combine(@"$(EngineDir)/Extras/VirtualProduction/TextureShare/TextureShareSDK");

		string    SDKSrcPath = Path.Combine(@"$(EngineDir)/Source/Programs/VirtualProduction/TextureShare/");

		string SDKHeadersDestPath = Path.Combine(SDKDestPath, "Source");

		// Copy Headers
		ImplPostBuildCopy(Path.Combine(SDKSrcPath, "Public", "*.h"), Path.Combine(SDKHeadersDestPath));
		ImplPostBuildCopy(Path.Combine(SDKSrcPath, "Public", "Serialize", "*.h"),  Path.Combine(SDKHeadersDestPath, "Serialize"));
		ImplPostBuildCopy(Path.Combine(SDKSrcPath, "Public", "Containers", "*.h"), Path.Combine(SDKHeadersDestPath, "Containers"));
		ImplPostBuildCopy(Path.Combine(SDKSrcPath, "Public", "Containers/UnrealEngine", "*.h"), Path.Combine(SDKHeadersDestPath, "Containers", "UnrealEngine"));

		string MainPluginSrcPath = Path.Combine(@"$(EngineDir)/Plugins/VirtualProduction/TextureShare/");

		ImplPostBuildCopy(Path.Combine(MainPluginSrcPath, "Source/TextureShare/Private", "Misc", "*.h"),           Path.Combine(SDKHeadersDestPath, "Misc"));
		ImplPostBuildCopy(Path.Combine(MainPluginSrcPath, "Source/TextureShareCore/Private", "Misc", "*.h"),       Path.Combine(SDKHeadersDestPath, "Misc"));
		ImplPostBuildCopy(Path.Combine(MainPluginSrcPath, "Source/TextureShareCore/Private", "Containers", "*.h"), Path.Combine(SDKHeadersDestPath, "Containers"));
		ImplPostBuildCopy(Path.Combine(MainPluginSrcPath, "Source/TextureShareCore/Private", "Serialize", "*.h"),  Path.Combine(SDKHeadersDestPath, "Serialize"));

		ImplPostBuildCopy(Path.Combine(MainPluginSrcPath, "Source/TextureShareDisplayCluster/Private", "Misc", "*.h"), Path.Combine(SDKHeadersDestPath, "Misc"));

		string SDKBinariesSrcPath = Path.Combine(@"$(EngineDir)/Binaries", Platform.ToString(), "TextureShareSDK");
		string SDKBinariesDestPath = Path.Combine(SDKDestPath, "Binaries", Platform.ToString());

		// Copy binaries
		ImplPostBuildCopy(Path.Combine(SDKBinariesSrcPath, "*.*"), SDKBinariesDestPath);
	}

	public TextureShareSDKTarget(TargetInfo Target)
		: base(Target)
	{
		DefaultBuildSettings = BuildSettingsVersion.Latest;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;

		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;

		// This is a library, we want it to produce a dy-lib
		bShouldCompileAsDLL = true;

		ExeBinariesSubFolder = @"TextureShareSDK";

		SolutionDirectory = "Programs/VirtualProduction/TextureShare";
		LaunchModuleName = "TextureShareSDK";

		// Lean and mean
		bBuildDeveloperTools = false;
		// Nor editor-only data
		bBuildWithEditorOnlyData = false;

		// Currently this module is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;

		// Logs and asserts are still useful to report results
		bUseLoggingInShipping = true;
		bUseChecksInShipping = true;

		// Whether to include ICU unicode/i18n support in Core
		bCompileICU = false;

		bUsesSlate = false;

		// Whether the final executable should export symbols.
		bHasExports = true;

		//@todo: approve this setting
		bBuildInSolutionByDefault = false;

		if (Platform == UnrealTargetPlatform.Win64)
		{
			AddWindowsPostBuildSteps();
		}
	}
}
